// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScriptDisassembler.cpp: Disassembler for Kismet bytecode.
=============================================================================*/

#include "ScriptDisassembler.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"

DEFINE_LOG_CATEGORY_STATIC(LogScriptDisassembler, Log, All);

/////////////////////////////////////////////////////
// FKismetBytecodeDisassembler

// Construct a disassembler that will output to the specified archive.
FKismetBytecodeDisassembler::FKismetBytecodeDisassembler(FOutputDevice& InAr)
	: Ar(InAr)
{
	InitTables();
}

// Disassemble all of the script code in a single structure.
void FKismetBytecodeDisassembler::DisassembleStructure(UFunction* Source)
{
	Script.Empty();
	Script.Append(Source->Script);

	int32 ScriptIndex = 0;
	while (ScriptIndex < Script.Num())
	{
		Ar.Logf(TEXT("Label_0x%X:"), ScriptIndex);

		AddIndent();
		SerializeExpr(ScriptIndex);
		DropIndent();
	}
}

// Disassemble all functions in any classes that have matching names.
void FKismetBytecodeDisassembler::DisassembleAllFunctionsInClasses(FOutputDevice& Ar, const FString& ClassnameSubstring)
{
	FKismetBytecodeDisassembler Disasm(Ar);
		
	for (TObjectIterator<UClass> ClassIter; ClassIter; ++ClassIter)
	{
		UClass* Class = *ClassIter;

		FString ClassName = Class->GetName();
		if (FCString::Strifind(*ClassName, *ClassnameSubstring))
		{
			Ar.Logf(TEXT("Processing class %s"), *ClassName);

			for (TFieldIterator<UFunction> FunctionIter(Class, EFieldIteratorFlags::ExcludeSuper); FunctionIter; ++FunctionIter)
			{
				UFunction* Function = *FunctionIter;
				FString FunctionName = Function->GetName();
				Ar.Logf(TEXT("  Processing function %s (%d bytes)"), *FunctionName, Function->Script.Num());

				Disasm.DisassembleStructure(Function);

				Ar.Logf(TEXT(""));
			}

			Ar.Logf(TEXT(""));
			Ar.Logf(TEXT("-----------"));
			Ar.Logf(TEXT(""));
		}

	}
}

EExprToken FKismetBytecodeDisassembler::SerializeExpr(int32& ScriptIndex)
{
	AddIndent();

	EExprToken Opcode = (EExprToken)Script[ScriptIndex];
	ScriptIndex++;

	ProcessCommon(ScriptIndex, Opcode);

	DropIndent();

	return Opcode;
}

int32 FKismetBytecodeDisassembler::ReadINT(int32& ScriptIndex)
{
	int32 Value = Script[ScriptIndex]; ++ScriptIndex;
	Value = Value | ((int32)Script[ScriptIndex] << 8); ++ScriptIndex;
	Value = Value | ((int32)Script[ScriptIndex] << 16); ++ScriptIndex;
	Value = Value | ((int32)Script[ScriptIndex] << 24); ++ScriptIndex;

	return Value;
}

uint64 FKismetBytecodeDisassembler::ReadQWORD(int32& ScriptIndex)
{
	uint64 Value = Script[ScriptIndex]; ++ScriptIndex;
	Value = Value | ((uint64)Script[ScriptIndex] << 8); ++ScriptIndex;
	Value = Value | ((uint64)Script[ScriptIndex] << 16); ++ScriptIndex;
	Value = Value | ((uint64)Script[ScriptIndex] << 24); ++ScriptIndex;
	Value = Value | ((uint64)Script[ScriptIndex] << 32); ++ScriptIndex;
	Value = Value | ((uint64)Script[ScriptIndex] << 40); ++ScriptIndex;
	Value = Value | ((uint64)Script[ScriptIndex] << 48); ++ScriptIndex;
	Value = Value | ((uint64)Script[ScriptIndex] << 56); ++ScriptIndex;

	return Value;
}

uint8 FKismetBytecodeDisassembler::ReadBYTE(int32& ScriptIndex)
{
	uint8 Value = Script[ScriptIndex]; ++ScriptIndex;

	return Value;
}

FString FKismetBytecodeDisassembler::ReadName(int32& ScriptIndex)
{
	const FScriptName ConstValue = *(FScriptName*)(Script.GetData() + ScriptIndex);
	ScriptIndex += sizeof(FScriptName);

	return ScriptNameToName(ConstValue).ToString();
}

uint16 FKismetBytecodeDisassembler::ReadWORD(int32& ScriptIndex)
{
	uint16 Value = Script[ScriptIndex]; ++ScriptIndex;
	Value = Value | ((uint16)Script[ScriptIndex] << 8); ++ScriptIndex;
	return Value;
}

float FKismetBytecodeDisassembler::ReadFLOAT(int32& ScriptIndex)
{
	union { float f; int32 i; } Result;
	Result.i = ReadINT(ScriptIndex);
	return Result.f;
}

double FKismetBytecodeDisassembler::ReadDOUBLE(int32& ScriptIndex)
{
	union { double d; int64 i; } Result;
	Result.i = ReadQWORD(ScriptIndex);
	return Result.d;
}

FVector FKismetBytecodeDisassembler::ReadFVECTOR(int32& ScriptIndex)
{
	FVector Vec;
	Vec.X = ReadDOUBLE(ScriptIndex);
	Vec.Y = ReadDOUBLE(ScriptIndex);
	Vec.Z = ReadDOUBLE(ScriptIndex);
	return Vec;
}

FRotator FKismetBytecodeDisassembler::ReadFROTATOR(int32& ScriptIndex)
{
	FRotator Rotator;
	Rotator.Pitch = ReadDOUBLE(ScriptIndex);
	Rotator.Yaw = ReadDOUBLE(ScriptIndex);
	Rotator.Roll = ReadDOUBLE(ScriptIndex);
	return Rotator;
}

FQuat FKismetBytecodeDisassembler::ReadFQUAT(int32& ScriptIndex)
{
	FQuat Quat;
	Quat.X = ReadDOUBLE(ScriptIndex);
	Quat.Y = ReadDOUBLE(ScriptIndex);
	Quat.Z = ReadDOUBLE(ScriptIndex);
	Quat.W = ReadDOUBLE(ScriptIndex);
	return Quat;
}

FTransform FKismetBytecodeDisassembler::ReadFTRANSFORM(int32& ScriptIndex)
{
	FTransform Transform;
	FQuat TmpRotation = ReadFQUAT(ScriptIndex);
	FVector TmpTranslation = ReadFVECTOR(ScriptIndex);
	FVector TmpScale = ReadFVECTOR(ScriptIndex);
	Transform.SetComponents(TmpRotation, TmpTranslation, TmpScale);
	return Transform;
}

CodeSkipSizeType FKismetBytecodeDisassembler::ReadSkipCount(int32& ScriptIndex)
{
#if SCRIPT_LIMIT_BYTECODE_TO_64KB
	return ReadWORD(ScriptIndex);
#else
	static_assert(sizeof(CodeSkipSizeType) == 4, "Update this code as size changed.");
	return ReadINT(ScriptIndex);
#endif
}

FString FKismetBytecodeDisassembler::ReadString(int32& ScriptIndex)
{
	const EExprToken Opcode = (EExprToken)Script[ScriptIndex++];

	switch (Opcode)
	{
	case EX_StringConst:
		return ReadString8(ScriptIndex);

	case EX_UnicodeStringConst:
		return ReadString16(ScriptIndex);

	default:
		checkf(false, TEXT("FKismetBytecodeDisassembler::ReadString - Unexpected opcode. Expected %d or %d, got %d"), (int)EX_StringConst, (int)EX_UnicodeStringConst, (int)Opcode);
		break;
	}

	return FString();
}

FString FKismetBytecodeDisassembler::ReadString8(int32& ScriptIndex)
{
	FString Result;

	do
	{
		Result += (ANSICHAR)ReadBYTE(ScriptIndex);
	}
	while (Script[ScriptIndex-1] != 0);

	return Result;
}

FString FKismetBytecodeDisassembler::ReadString16(int32& ScriptIndex)
{
	FString Result;

	do
	{
		Result += (TCHAR)ReadWORD(ScriptIndex);
	}
	while ((Script[ScriptIndex-1] != 0) || (Script[ScriptIndex-2] != 0));

	return Result;
}

void FKismetBytecodeDisassembler::ProcessCommon(int32& ScriptIndex, EExprToken Opcode)
{
	static const TCHAR* CastNameTable[CST_Max] = {
		TEXT("ObjectToInterface"),
		TEXT("ObjectToBool"),
		TEXT("InterfaceToBool"),
		TEXT("DoubleToFloat"),
		TEXT("FloatToDouble"),
	};

	auto PrintVariable = [&ScriptIndex, Opcode, this](FStringView VariableDescription)
	{
		FProperty* PropertyPtr = ReadPointer<FProperty>(ScriptIndex);
		FString PropertyName = TEXT("(null)");
		FString PropertyType = TEXT("(null)");
		FString ParameterType;

		if (PropertyPtr)
		{
			PropertyName = PropertyPtr->GetName();

			FString ExtendedPropertyType;
			PropertyType = PropertyPtr->GetCPPType(&ExtendedPropertyType);
			PropertyType += ExtendedPropertyType;

			if (PropertyPtr->HasAnyPropertyFlags(CPF_ParmFlags))
			{
				ParameterType = TEXT("(");
				if (PropertyPtr->HasAnyPropertyFlags(CPF_Parm))
				{
					ParameterType += TEXT("Parameter,");
				}

				if (PropertyPtr->HasAnyPropertyFlags(CPF_OutParm))
				{
					ParameterType += TEXT("Out,");
				}

				if (PropertyPtr->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					ParameterType += TEXT("Return,");
				}

				if (PropertyPtr->HasAnyPropertyFlags(CPF_RequiredParm))
				{
					ParameterType += TEXT("Required,");
				}

				if (PropertyPtr->HasAnyPropertyFlags(CPF_ReferenceParm))
				{
					ParameterType += TEXT("Reference,");
				}

				if (PropertyPtr->HasAnyPropertyFlags(CPF_ConstParm))
				{
					ParameterType += TEXT("Const,");
				}

				int32 LastCommaLocation = ParameterType.Len() - 1;
				check(LastCommaLocation > 0);
				ParameterType[LastCommaLocation] = TCHAR(')');
			}
		}

		FString Output = 
			FString::Printf(TEXT("%s $%X: %s of type %s named %s."), *Indents, (int32)Opcode, VariableDescription.GetData(), *PropertyType, *PropertyName);

		if (ParameterType.Len() > 0)
		{
			Output += FString::Printf(TEXT(" Parameter flags: %s."), *ParameterType);
		}

		Ar.Logf(TEXT("%s"), *Output);
	};

	switch (Opcode)
	{
	case EX_Cast:
		{
			// A type conversion.
			uint8 ConversionType = ReadBYTE(ScriptIndex);
			check(CastNameTable[ConversionType] != nullptr);
			Ar.Logf(TEXT("%s $%X: Cast of type %d (%s)"), *Indents, (int32)Opcode, ConversionType, CastNameTable[ConversionType]);
			AddIndent();

			Ar.Logf(TEXT("%s Argument:"), *Indents);
			SerializeExpr(ScriptIndex);

			DropIndent();
			break;
		}
	case EX_SetSet:
		{
 			Ar.Logf(TEXT("%s $%X: set set"), *Indents, (int32)Opcode);
			SerializeExpr(ScriptIndex);
			ReadINT(ScriptIndex);
 			while (SerializeExpr(ScriptIndex) != EX_EndSet)
 			{
 				// Set contents
 			}
 			break;
		}
	case EX_EndSet:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndSet"), *Indents, (int32)Opcode);
			break;
		}
	case EX_SetConst:
		{
			FProperty* InnerProp = ReadPointer<FProperty>(ScriptIndex);
			int32 Num = ReadINT(ScriptIndex);
 			Ar.Logf(TEXT("%s $%X: set set const - elements number: %d, inner property: %s"), *Indents, (int32)Opcode, Num, *GetNameSafe(InnerProp));
 			while (SerializeExpr(ScriptIndex) != EX_EndSetConst)
 			{
 				// Set contents
 			}
 			break;
		}
	case EX_EndSetConst:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndSetConst"), *Indents, (int32)Opcode);
			break;
		}
	case EX_SetMap:
		{
 			Ar.Logf(TEXT("%s $%X: set map"), *Indents, (int32)Opcode);
			SerializeExpr(ScriptIndex);
 			ReadINT(ScriptIndex);
 			while (SerializeExpr(ScriptIndex) != EX_EndMap)
 			{
 				// Map contents
 			}
 			break;
		}
	case EX_EndMap:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndMap"), *Indents, (int32)Opcode);
			break;
		}
	case EX_MapConst:
		{
			FProperty* KeyProp = ReadPointer<FProperty>(ScriptIndex);
			FProperty* ValProp = ReadPointer<FProperty>(ScriptIndex);
			int32 Num = ReadINT(ScriptIndex);
 			Ar.Logf(TEXT("%s $%X: set map const - elements number: %d, key property: %s, val property: %s"), *Indents, (int32)Opcode, Num, *GetNameSafe(KeyProp), *GetNameSafe(ValProp));
 			while (SerializeExpr(ScriptIndex) != EX_EndMapConst)
 			{
 				// Map contents
 			}
 			break;
		}
	case EX_EndMapConst:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndMapConst"), *Indents, (int32)Opcode);
			break;
		}
	case EX_ObjToInterfaceCast:
		{
			// A conversion from an object variable to a native interface variable.
			// We use a different bytecode to avoid the branching each time we process a cast token

			// the interface class to convert to
			UClass* InterfaceClass = ReadPointer<UClass>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: ObjToInterfaceCast to %s"), *Indents, (int32)Opcode, *InterfaceClass->GetName());

			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_CrossInterfaceCast:
		{
			// A conversion from one interface variable to a different interface variable.
			// We use a different bytecode to avoid the branching each time we process a cast token

			// the interface class to convert to
			UClass* InterfaceClass = ReadPointer<UClass>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: InterfaceToInterfaceCast to %s"), *Indents, (int32)Opcode, *InterfaceClass->GetName());

			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_InterfaceToObjCast:
		{
			// A conversion from an interface variable to a object variable.
			// We use a different bytecode to avoid the branching each time we process a cast token

			// the interface class to convert to
			UClass* ObjectClass = ReadPointer<UClass>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: InterfaceToObjCast to %s"), *Indents, (int32)Opcode, *ObjectClass->GetName());

			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_Let:
		{
			Ar.Logf(TEXT("%s $%X: Let (Variable = Expression)"), *Indents, (int32)Opcode);
			AddIndent();

			ReadPointer<FProperty>(ScriptIndex);

			// Variable expr.
			Ar.Logf(TEXT("%s Variable:"), *Indents);
			SerializeExpr( ScriptIndex );

			// Assignment expr.
			Ar.Logf(TEXT("%s Expression:"), *Indents);
			SerializeExpr( ScriptIndex );

			DropIndent();
			break;
		}
	case EX_LetObj:
	case EX_LetWeakObjPtr:
		{
			if( Opcode == EX_LetObj )
			{
				Ar.Logf(TEXT("%s $%X: Let Obj (Variable = Expression)"), *Indents, (int32)Opcode);
			}
			else
			{
				Ar.Logf(TEXT("%s $%X: Let WeakObjPtr (Variable = Expression)"), *Indents, (int32)Opcode);
			}
			AddIndent();

			// Variable expr.
			Ar.Logf(TEXT("%s Variable:"), *Indents);
			SerializeExpr( ScriptIndex );

			// Assignment expr.
			Ar.Logf(TEXT("%s Expression:"), *Indents);
			SerializeExpr( ScriptIndex );

			DropIndent();
			break;
		}
	case EX_LetBool:
		{
			Ar.Logf(TEXT("%s $%X: LetBool (Variable = Expression)"), *Indents, (int32)Opcode);
			AddIndent();

			// Variable expr.
			Ar.Logf(TEXT("%s Variable:"), *Indents);
			SerializeExpr( ScriptIndex );

			// Assignment expr.
			Ar.Logf(TEXT("%s Expression:"), *Indents);
			SerializeExpr( ScriptIndex );

			DropIndent();
			break;
		}
	case EX_LetValueOnPersistentFrame:
		{
			Ar.Logf(TEXT("%s $%X: LetValueOnPersistentFrame"), *Indents, (int32)Opcode);
			AddIndent();

			auto Prop = ReadPointer<FProperty>(ScriptIndex);
			Ar.Logf(TEXT("%s Destination variable: %s, offset: %d"), *Indents, *GetNameSafe(Prop), 
				Prop ? Prop->GetOffset_ForDebug() : 0);
			
			Ar.Logf(TEXT("%s Expression:"), *Indents);
			SerializeExpr(ScriptIndex);

			DropIndent();

			break;
		}
	case EX_StructMemberContext:
		{
			Ar.Logf(TEXT("%s $%X: Struct member context "), *Indents, (int32)Opcode);
			AddIndent();

			FProperty* Prop = ReadPointer<FProperty>(ScriptIndex);

			if (!Prop) // Try to continue even if the code is broken
			{
				Ar.Logf(TEXT("%s Member named <NULL> @ offset <UNKNOWN>"), *Indents);
			}
			else
			{
				Ar.Logf(TEXT("%s Member named %s @ offset %d"), *Indents, *(Prop->GetName()),
					Prop->GetOffset_ForDebug()); // although that isn't a UFunction, we are not going to indirect the props of a struct, so this should be fine
			}

			Ar.Logf(TEXT("%s Expression to struct:"), *Indents);
			SerializeExpr( ScriptIndex );

			DropIndent();

			break;
		}
	case EX_LetDelegate:
		{
			Ar.Logf(TEXT("%s $%X: LetDelegate (Variable = Expression)"), *Indents, (int32)Opcode);
			AddIndent();

			// Variable expr.
			Ar.Logf(TEXT("%s Variable:"), *Indents);
			SerializeExpr( ScriptIndex );
				
			// Assignment expr.
			Ar.Logf(TEXT("%s Expression:"), *Indents);
			SerializeExpr( ScriptIndex );

			DropIndent();
			break;
		}
	case EX_LocalVirtualFunction:
		{
			FString FunctionName = ReadName(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: Local Virtual Script Function named %s"), *Indents, (int32)Opcode, *FunctionName);

			while (SerializeExpr(ScriptIndex) != EX_EndFunctionParms)
			{
			}
			break;
		}
	case EX_LocalFinalFunction:
		{
			UStruct* StackNode = ReadPointer<UStruct>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: Local Final Script Function (stack node %s::%s)"), *Indents, (int32)Opcode, StackNode ? *StackNode->GetOuter()->GetName() : TEXT("(null)"), StackNode ? *StackNode->GetName() : TEXT("(null)"));

			while (SerializeExpr(ScriptIndex) != EX_EndFunctionParms)
			{
				// Params
			}
			break;
		}
	case EX_LetMulticastDelegate:
		{
			Ar.Logf(TEXT("%s $%X: LetMulticastDelegate (Variable = Expression)"), *Indents, (int32)Opcode);
			AddIndent();

			// Variable expr.
			Ar.Logf(TEXT("%s Variable:"), *Indents);
			SerializeExpr( ScriptIndex );
				
			// Assignment expr.
			Ar.Logf(TEXT("%s Expression:"), *Indents);
			SerializeExpr( ScriptIndex );

			DropIndent();
			break;
		}
	case EX_ComputedJump:
		{
			Ar.Logf(TEXT("%s $%X: Computed Jump, offset specified by expression:"), *Indents, (int32)Opcode);

			AddIndent();
			SerializeExpr( ScriptIndex );
			DropIndent();

			break;
		}
	case EX_Jump:
		{
			CodeSkipSizeType SkipCount = ReadSkipCount(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: Jump to offset 0x%X"), *Indents, (int32)Opcode, SkipCount);
			break;
		}
	case EX_LocalVariable:
		{
			PrintVariable(TEXT("Local variable"));
			break;
		}
	case EX_DefaultVariable:
		{
			PrintVariable(TEXT("Default variable"));
			break;
		}
	case EX_InstanceVariable:
		{
			PrintVariable(TEXT("Instance variable"));
			break;
		}
	case EX_LocalOutVariable:
		{
			PrintVariable(TEXT("Local out variable"));
			break;
		}
	case EX_ClassSparseDataVariable:
		{
			PrintVariable(TEXT("Class sparse data variable"));
			break;
		}
	case EX_InterfaceContext:
		{
			Ar.Logf(TEXT("%s $%X: EX_InterfaceContext:"), *Indents, (int32)Opcode);
			SerializeExpr(ScriptIndex);
			break;
		}
	case EX_DeprecatedOp4A:
		{
			Ar.Logf(TEXT("%s $%X: This opcode has been removed and does nothing."), *Indents, (int32)Opcode);
			break;
		}
	case EX_Nothing:
		{
			Ar.Logf(TEXT("%s $%X: EX_Nothing"), *Indents, (int32)Opcode);
			break;
		}
	case EX_NothingInt32:
		{
			int32 Value = ReadINT(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: EX_NothingInt32 %d"), *Indents, (int32)Opcode, Value);
			break;
		}
	case EX_EndOfScript:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndOfScript"), *Indents, (int32)Opcode);
			break;
		}
	case EX_EndFunctionParms:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndFunctionParms"), *Indents, (int32)Opcode);
			break;
		}
	case EX_EndStructConst:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndStructConst"), *Indents, (int32)Opcode);
			break;
		}
	case EX_EndArray:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndArray"), *Indents, (int32)Opcode);
			break;
		}
	case EX_EndArrayConst:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndArrayConst"), *Indents, (int32)Opcode);
			break;
		}
	case EX_IntZero:
		{
			Ar.Logf(TEXT("%s $%X: EX_IntZero"), *Indents, (int32)Opcode);
			break;
		}
	case EX_IntOne:
		{
			Ar.Logf(TEXT("%s $%X: EX_IntOne"), *Indents, (int32)Opcode);
			break;
		}
	case EX_True:
		{
			Ar.Logf(TEXT("%s $%X: EX_True"), *Indents, (int32)Opcode);
			break;
		}
	case EX_False:
		{
			Ar.Logf(TEXT("%s $%X: EX_False"), *Indents, (int32)Opcode);
			break;
		}
	case EX_NoObject:
		{
			Ar.Logf(TEXT("%s $%X: EX_NoObject"), *Indents, (int32)Opcode);
			break;
		}
	case EX_NoInterface:
		{
			Ar.Logf(TEXT("%s $%X: EX_NoObject"), *Indents, (int32)Opcode);
			break;
		}
	case EX_Self:
		{
			Ar.Logf(TEXT("%s $%X: EX_Self"), *Indents, (int32)Opcode);
			break;
		}
	case EX_EndParmValue:
		{
			Ar.Logf(TEXT("%s $%X: EX_EndParmValue"), *Indents, (int32)Opcode);
			break;
		}
	case EX_Return:
		{
			Ar.Logf(TEXT("%s $%X: Return expression"), *Indents, (int32)Opcode);

			SerializeExpr( ScriptIndex ); // Return expression.
			break;
		}
	case EX_CallMath:
		{
			UStruct* StackNode = ReadPointer<UStruct>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: Call Math (stack node %s::%s)"), *Indents, (int32)Opcode, *GetNameSafe(StackNode ? StackNode->GetOuter() : nullptr), *GetNameSafe(StackNode));

			while (SerializeExpr(ScriptIndex) != EX_EndFunctionParms)
			{
				// Params
			}
			break;
		}
	case EX_FinalFunction:
		{
			UStruct* StackNode = ReadPointer<UStruct>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: Final Function (stack node %s::%s)"), *Indents, (int32)Opcode, StackNode ? *StackNode->GetOuter()->GetName() : TEXT("(null)"), StackNode ? *StackNode->GetName() : TEXT("(null)"));

			while (SerializeExpr(ScriptIndex) != EX_EndFunctionParms)
			{
				// Params
			}
			break;
		}
	case EX_CallMulticastDelegate:
		{
			UStruct* StackNode = ReadPointer<UStruct>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: CallMulticastDelegate (signature %s::%s) delegate:"), *Indents, (int32)Opcode, StackNode ? *StackNode->GetOuter()->GetName() : TEXT("(null)"), StackNode ? *StackNode->GetName() : TEXT("(null)"));
			SerializeExpr( ScriptIndex );
			Ar.Logf(TEXT("Params:"));
			while (SerializeExpr(ScriptIndex) != EX_EndFunctionParms)
			{
				// Params
			}
			break;
		}
	case EX_VirtualFunction:
		{
			FString FunctionName = ReadName(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: Virtual Function named %s"), *Indents, (int32)Opcode, *FunctionName);

			while (SerializeExpr(ScriptIndex) != EX_EndFunctionParms)
			{
			}
			break;
		}
	case EX_ClassContext:
	case EX_Context:
	case EX_Context_FailSilent:
		{
			Ar.Logf(TEXT("%s $%X: %s"), *Indents, (int32)Opcode, Opcode == EX_ClassContext ? TEXT("Class Context") : TEXT("Context"));
			AddIndent();

			// Object expression.
			Ar.Logf(TEXT("%s ObjectExpression:"), *Indents);
			SerializeExpr( ScriptIndex );

			if (Opcode == EX_Context_FailSilent)
			{
				Ar.Logf(TEXT(" Can fail silently on access none "));
			}

			// Code offset for NULL expressions.
			CodeSkipSizeType SkipCount = ReadSkipCount(ScriptIndex);
			Ar.Logf(TEXT("%s Skip 0x%X bytes to offset 0x%X"), *Indents, SkipCount, ScriptIndex + sizeof(FField*) + SkipCount);

			// Property corresponding to the r-value data, in case the l-value needs to be mem-zero'd
			FField* Field = ReadPointer<FField>(ScriptIndex);
			Ar.Logf(TEXT("%s R-Value Property: %s"), *Indents, Field ? *Field->GetName() : TEXT("(null)"));

			// Context expression.
			Ar.Logf(TEXT("%s ContextExpression:"), *Indents);
			SerializeExpr( ScriptIndex );

			DropIndent();
			break;
		}
	case EX_IntConst:
		{
			int32 ConstValue = ReadINT(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal int32 %d"), *Indents, (int32)Opcode, ConstValue);
			break;
		}
	case EX_Int64Const:
		{
			int64 ConstValue = ReadQWORD(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal int64 0x%" INT64_X_FMT), *Indents, (int32)Opcode, ConstValue);
			break;
		}
	case EX_UInt64Const:
		{
			uint64 ConstValue = ReadQWORD(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal uint64 0x%" UINT64_X_FMT), *Indents, (int32)Opcode, ConstValue);
			break;
		}
	case EX_SkipOffsetConst:
		{
			CodeSkipSizeType ConstValue = ReadSkipCount(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal CodeSkipSizeType 0x%X"), *Indents, (int32)Opcode, ConstValue);
			break;
		}
	case EX_FloatConst:
		{
			float ConstValue = ReadFLOAT(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal float %f"), *Indents, (int32)Opcode, ConstValue);
			break;
		}
	case EX_DoubleConst:
		{
			double ConstValue = ReadDOUBLE(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal double %lf"), *Indents, (int32)Opcode, ConstValue);
			break;
		}
	case EX_StringConst:
		{
			FString ConstValue = ReadString8(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal ansi string \"%s\""), *Indents, (int32)Opcode, *ConstValue);
			break;
		}
	case EX_UnicodeStringConst:
		{
			FString ConstValue = ReadString16(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal unicode string \"%s\""), *Indents, (int32)Opcode, *ConstValue);
			break;
		}
	case EX_TextConst:
		{
			// What kind of text are we dealing with?
			const EBlueprintTextLiteralType TextLiteralType = (EBlueprintTextLiteralType)Script[ScriptIndex++];

			switch (TextLiteralType)
			{
			case EBlueprintTextLiteralType::Empty:
				{
					Ar.Logf(TEXT("%s $%X: literal text - empty"), *Indents, (int32)Opcode);
				}
				break;

			case EBlueprintTextLiteralType::LocalizedText:
			case EBlueprintTextLiteralType::LocalizedTextWithNotes:
				{
					const FString SourceString = ReadString(ScriptIndex);
					const FString KeyString = ReadString(ScriptIndex);
					const FString Namespace = ReadString(ScriptIndex);
					if (TextLiteralType == EBlueprintTextLiteralType::LocalizedTextWithNotes)
					{
						const FString DevNotes = ReadString(ScriptIndex);
						Ar.Logf(TEXT("%s $%X: literal text - localized text { namespace: \"%s\", key: \"%s\", source: \"%s\", devnotes: \"%s\" }"), *Indents, (int32)Opcode, *Namespace, *KeyString, *SourceString, *DevNotes);
					}
					else
					{
						Ar.Logf(TEXT("%s $%X: literal text - localized text { namespace: \"%s\", key: \"%s\", source: \"%s\" }"), *Indents, (int32)Opcode, *Namespace, *KeyString, *SourceString);
					}
				}
				break;

			case EBlueprintTextLiteralType::InvariantText:
				{
					const FString SourceString = ReadString(ScriptIndex);
					Ar.Logf(TEXT("%s $%X: literal text - invariant text: \"%s\""), *Indents, (int32)Opcode, *SourceString);
				}
				break;

			case EBlueprintTextLiteralType::LiteralString:
				{
					const FString SourceString = ReadString(ScriptIndex);
					Ar.Logf(TEXT("%s $%X: literal text - literal string: \"%s\""), *Indents, (int32)Opcode, *SourceString);
				}
				break;

			case EBlueprintTextLiteralType::StringTableEntry:
				{
					ReadPointer<UObject>(ScriptIndex); // String Table asset (if any)
					const FString TableIdString = ReadString(ScriptIndex);
					const FString KeyString = ReadString(ScriptIndex);
					Ar.Logf(TEXT("%s $%X: literal text - string table entry { tableid: \"%s\", key: \"%s\" }"), *Indents, (int32)Opcode, *TableIdString, *KeyString);
				}
				break;

			default:
				checkf(false, TEXT("Unknown EBlueprintTextLiteralType! Please update FKismetBytecodeDisassembler::ProcessCommon to handle this type of text."));
				break;
			}
			break;
		}
	case EX_PropertyConst:
		{
			FProperty* Pointer = ReadPointer<FProperty>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: EX_PropertyConst (%p:%s)"), *Indents, (int32)Opcode, Pointer, Pointer ? *Pointer->GetName() : TEXT("(null)"));
			break;
		}
	case EX_ObjectConst:
		{
			UObject* Pointer = ReadPointer<UObject>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: EX_ObjectConst (%p:%s)"), *Indents, (int32)Opcode, Pointer, Pointer ? (Pointer->IsValidLowLevel() ? *Pointer->GetFullName() : TEXT("(not a valid object)")) : TEXT("(null)"));
			break;
		}
	case EX_SoftObjectConst:
		{
			Ar.Logf(TEXT("%s $%X: EX_SoftObjectConst"), *Indents, (int32)Opcode);
			SerializeExpr(ScriptIndex);
			break;
		}
	case EX_FieldPathConst:
		{
			Ar.Logf(TEXT("%s $%X: EX_FieldPathConst"), *Indents, (int32)Opcode);
			SerializeExpr(ScriptIndex);
			break;
		}
	case EX_NameConst:
		{
			FString ConstValue = ReadName(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal name %s"), *Indents, (int32)Opcode, *ConstValue);
			break;
		}
	case EX_RotationConst:
		{
			const FRotator Rotator = ReadFROTATOR(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal rotation (%f,%f,%f)"), *Indents, (int32)Opcode, Rotator.Pitch, Rotator.Yaw, Rotator.Roll);
			break;
		}
	case EX_VectorConst:
		{
			FVector Vec = ReadFVECTOR(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal vector (%f,%f,%f)"), *Indents, (int32)Opcode, Vec.X, Vec.Y, Vec.Z);
			break;
		}
	case EX_Vector3fConst:
		{
			FVector3f Vec = (FVector3f)ReadFVECTOR(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal float vector (%f,%f,%f)"), *Indents, (int32)Opcode, Vec.X, Vec.Y, Vec.Z);
			break;
		}
	case EX_TransformConst:
		{
			const FTransform Transform = ReadFTRANSFORM(ScriptIndex);
			const FQuat Rotation = Transform.GetRotation();
			const FVector Translation = Transform.GetTranslation();
			const FVector Scale = Transform.GetScale3D();
			Ar.Logf(TEXT("%s $%X: literal transform R(%f,%f,%f,%f) T(%f,%f,%f) S(%f,%f,%f)"),
				    *Indents,
					(int32)Opcode, 
					Rotation.X, Rotation.Y, Rotation.Z, Rotation.W,
					Translation.X, Translation.Y, Translation.Z,
					Scale.X, Scale.Y, Scale.Z);
			break;
		}
	case EX_StructConst:
		{
			UScriptStruct* Struct = ReadPointer<UScriptStruct>(ScriptIndex);
			int32 SerializedSize = ReadINT(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal struct %s (serialized size: %d)"), *Indents, (int32)Opcode, *Struct->GetName(), SerializedSize);
			while (SerializeExpr(ScriptIndex) != EX_EndStructConst)
			{
				// struct contents
			}
			break;
		}
	case EX_SetArray:
		{
 			Ar.Logf(TEXT("%s $%X: set array"), *Indents, (int32)Opcode);
			SerializeExpr(ScriptIndex);
 			while (SerializeExpr(ScriptIndex) != EX_EndArray)
 			{
 				// Array contents
 			}
 			break;
		}
	case EX_ArrayConst:
		{
			FProperty* InnerProp = ReadPointer<FProperty>(ScriptIndex);
			int32 Num = ReadINT(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: set array const - elements number: %d, inner property: %s"), *Indents, (int32)Opcode, Num, *GetNameSafe(InnerProp));
			while (SerializeExpr(ScriptIndex) != EX_EndArrayConst)
			{
				// Array contents
			}
			break;
		}
	case EX_BitFieldConst:
		{
			FProperty* BitProperty = ReadPointer<FProperty>(ScriptIndex);
			uint8 ConstValue = ReadBYTE(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: set bit property %s to value %d"), *Indents, (int32)Opcode, *GetNameSafe(BitProperty), ConstValue);
			break;
		}
	case EX_ByteConst:
		{
			uint8 ConstValue = ReadBYTE(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal byte %d"), *Indents, (int32)Opcode, ConstValue);
			break;
		}
	case EX_IntConstByte:
		{
			int32 ConstValue = ReadBYTE(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: literal int %d"), *Indents, (int32)Opcode, ConstValue);
			break;
		}
	case EX_MetaCast:
		{
			UClass* Class = ReadPointer<UClass>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: MetaCast to %s of expr:"), *Indents, (int32)Opcode, *Class->GetName());
			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_DynamicCast:
		{
			UClass* Class = ReadPointer<UClass>(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: DynamicCast to %s of expr:"), *Indents, (int32)Opcode, *Class->GetName());
			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_JumpIfNot:
		{
			// Code offset.
			CodeSkipSizeType SkipCount = ReadSkipCount(ScriptIndex);
				
			Ar.Logf(TEXT("%s $%X: Jump to offset 0x%X if not expr:"), *Indents, (int32)Opcode, SkipCount);

			// Boolean expr.
			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_Assert:
		{
			uint16 LineNumber = ReadWORD(ScriptIndex);
			uint8 InDebugMode = ReadBYTE(ScriptIndex);

			Ar.Logf(TEXT("%s $%X: assert at line %d, in debug mode = %d with expr:"), *Indents, (int32)Opcode, LineNumber, InDebugMode);
			SerializeExpr( ScriptIndex ); // Assert expr.
			break;
		}
	case EX_Skip:
		{
			CodeSkipSizeType W = ReadSkipCount(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: possibly skip 0x%X bytes of expr:"), *Indents, (int32)Opcode, W);

			// Expression to possibly skip.
			SerializeExpr( ScriptIndex );

			break;
		}
	case EX_InstanceDelegate:
		{
			// the name of the function assigned to the delegate.
			FString FuncName = ReadName(ScriptIndex);

			Ar.Logf(TEXT("%s $%X: instance delegate function named %s"), *Indents, (int32)Opcode, *FuncName);
			break;
		}
	case EX_AddMulticastDelegate:
		{
			Ar.Logf(TEXT("%s $%X: Add MC delegate"), *Indents, (int32)Opcode);
			SerializeExpr( ScriptIndex );
			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_RemoveMulticastDelegate:
		{
			Ar.Logf(TEXT("%s $%X: Remove MC delegate"), *Indents, (int32)Opcode);
			SerializeExpr( ScriptIndex );
			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_ClearMulticastDelegate:
		{
			Ar.Logf(TEXT("%s $%X: Clear MC delegate"), *Indents, (int32)Opcode);
			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_BindDelegate:
		{
			// the name of the function assigned to the delegate.
			FString FuncName = ReadName(ScriptIndex);

			Ar.Logf(TEXT("%s $%X: BindDelegate '%s' "), *Indents, (int32)Opcode, *FuncName);

			Ar.Logf(TEXT("%s Delegate:"), *Indents);
			SerializeExpr( ScriptIndex );

			Ar.Logf(TEXT("%s Object:"), *Indents);
			SerializeExpr( ScriptIndex );

			break;
		}
	case EX_PushExecutionFlow:
		{
			CodeSkipSizeType SkipCount = ReadSkipCount(ScriptIndex);
			Ar.Logf(TEXT("%s $%X: FlowStack.Push(0x%X);"), *Indents, (int32)Opcode, SkipCount);
			break;
		}
	case EX_PopExecutionFlow:
		{
			Ar.Logf(TEXT("%s $%X: if (FlowStack.Num()) { jump to statement at FlowStack.Pop(); } else { ERROR!!! }"), *Indents, (int32)Opcode);
			break;
		}
	case EX_PopExecutionFlowIfNot:
		{
			Ar.Logf(TEXT("%s $%X: if (!condition) { if (FlowStack.Num()) { jump to statement at FlowStack.Pop(); } else { ERROR!!! } }"), *Indents, (int32)Opcode);
			// Boolean expr.
			SerializeExpr( ScriptIndex );
			break;
		}
	case EX_Breakpoint:
		{
			Ar.Logf(TEXT("%s $%X: <<< BREAKPOINT >>>"), *Indents, (int32)Opcode);
			break;
		}
	case EX_WireTracepoint:
		{
			Ar.Logf(TEXT("%s $%X: .. wire debug site .."), *Indents, (int32)Opcode);
			break;
		}
	case EX_InstrumentationEvent:
		{
			const uint8 EventType = ReadBYTE(ScriptIndex);
			switch (EventType)
			{
				case EScriptInstrumentation::InlineEvent:
					Ar.Logf(TEXT("%s $%X: .. instrumented inline event .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::Stop:
					Ar.Logf(TEXT("%s $%X: .. instrumented event stop .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::PureNodeEntry:
					Ar.Logf(TEXT("%s $%X: .. instrumented pure node entry site .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::NodeDebugSite:
					Ar.Logf(TEXT("%s $%X: .. instrumented debug site .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::NodeEntry:
					Ar.Logf(TEXT("%s $%X: .. instrumented wire entry site .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::NodeExit:
					Ar.Logf(TEXT("%s $%X: .. instrumented wire exit site .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::PushState:
					Ar.Logf(TEXT("%s $%X: .. push execution state .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::RestoreState:
					Ar.Logf(TEXT("%s $%X: .. restore execution state .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::ResetState:
					Ar.Logf(TEXT("%s $%X: .. reset execution state .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::SuspendState:
					Ar.Logf(TEXT("%s $%X: .. suspend execution state .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::PopState:
					Ar.Logf(TEXT("%s $%X: .. pop execution state .."), *Indents, (int32)Opcode);
					break;
				case EScriptInstrumentation::TunnelEndOfThread:
					Ar.Logf(TEXT("%s $%X: .. tunnel end of thread .."), *Indents, (int32)Opcode);
					break;
			}
			break;
		}
	case EX_Tracepoint:
		{
			Ar.Logf(TEXT("%s $%X: .. debug site .."), *Indents, (int32)Opcode);
			break;
		}
	case EX_SwitchValue:
		{
			const uint16 NumCases = ReadWORD(ScriptIndex);
			const CodeSkipSizeType AfterSkip = ReadSkipCount(ScriptIndex);

			Ar.Logf(TEXT("%s $%X: Switch Value %d cases, end in 0x%X"), *Indents, (int32)Opcode, NumCases, AfterSkip);
			AddIndent();
			Ar.Logf(TEXT("%s Index:"), *Indents);
			SerializeExpr(ScriptIndex);

			for (uint16 CaseIndex = 0; CaseIndex < NumCases; ++CaseIndex)
			{
				Ar.Logf(TEXT("%s [%d] Case Index (label: 0x%X):"), *Indents, CaseIndex, ScriptIndex);
				SerializeExpr(ScriptIndex);	// case index value term
				const CodeSkipSizeType OffsetToNextCase = ReadSkipCount(ScriptIndex);
				Ar.Logf(TEXT("%s [%d] Offset to the next case: 0x%X"), *Indents, CaseIndex, OffsetToNextCase);
				Ar.Logf(TEXT("%s [%d] Case Result:"), *Indents, CaseIndex);
				SerializeExpr(ScriptIndex);	// case term
			}

			Ar.Logf(TEXT("%s Default result (label: 0x%X):"), *Indents, ScriptIndex);
			SerializeExpr(ScriptIndex);
			Ar.Logf(TEXT("%s (label: 0x%X)"), *Indents, ScriptIndex);
			DropIndent();
			break;
		}
	case EX_ArrayGetByRef:
		{
			Ar.Logf(TEXT("%s $%X: Array Get-by-Ref Index"), *Indents, (int32)Opcode);
			AddIndent();
			SerializeExpr(ScriptIndex);
			SerializeExpr(ScriptIndex);
			DropIndent();
			break;
		}
	case EX_AutoRtfmTransact:
		{
			// Code offset.
			int32 Value = ReadINT(ScriptIndex);				
			CodeSkipSizeType SkipCount = ReadSkipCount(ScriptIndex);

			Ar.Logf(TEXT("%s $%X: AutoRtfmTransact %d to offset 0x%X"), *Indents, (int32)Opcode, Value, SkipCount);

			while (SerializeExpr(ScriptIndex) != EX_AutoRtfmStopTransact)
			{
				// Params
			}
			break;
		}
	case EX_AutoRtfmStopTransact:
		{
			int32 Value = ReadINT(ScriptIndex);
			EAutoRtfmStopTransactMode Mode = EAutoRtfmStopTransactMode(ReadBYTE(ScriptIndex));

			const TCHAR* ModeText = TEXT("");
			switch(Mode)
			{
			case EAutoRtfmStopTransactMode::GracefulExit: ModeText = TEXT("GracefulExit"); break;
			case EAutoRtfmStopTransactMode::AbortingExit: ModeText = TEXT("AbortingExit"); break;
			case EAutoRtfmStopTransactMode::AbortingExitAndAbortParent: ModeText = TEXT("AbortingExitAndAbortParent"); break;
			}

			Ar.Logf(TEXT("%s $%X: EX_AutoRtfmStopTransact (%s) %d"), *Indents, (int32)Opcode, ModeText, Value);
			break;
		}
	case EX_AutoRtfmAbortIfNot:
		{
			Ar.Logf(TEXT("%s $%X: EX_AutoRtfmAbortIfNot"), *Indents, (int32)Opcode);
			SerializeExpr(ScriptIndex);
			break;
		}
	case EX_AutoRtfmAbort:
		{
			Ar.Logf(TEXT("%s $%X: EX_AutoRtfmAbort"), *Indents, (int32)Opcode);
			break;
		}
	default:
		{
			// This should never occur.
			UE_LOGF(LogScriptDisassembler, Warning, "Unknown bytecode 0x%02X; ignoring it", (uint8)Opcode );
			break;
		}
	}
}

void FKismetBytecodeDisassembler::InitTables()
{
}
