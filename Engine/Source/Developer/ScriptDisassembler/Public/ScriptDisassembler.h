// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScriptDisassembler.h: Disassembler for Kismet bytecode.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/Script.h"

class FOutputDevice;
class UFunction;

/**
 * Kismet bytecode disassembler; Can be used to create a human readable version
 * of Kismet bytecode for a specified structure or class.
 */
class FKismetBytecodeDisassembler
{
private:
	TArray<uint8> Script;
	FString Indents;
	FOutputDevice& Ar;
public:
	/**
	 * Construct a disassembler that will output to the specified archive.
	 *
	 * @param	InAr	The archive to emit disassembled bytecode to.
	 */
	SCRIPTDISASSEMBLER_API FKismetBytecodeDisassembler(FOutputDevice& InAr);

	/**
	 * Disassemble all of the script code in a single structure.
	 *
	 * @param [in,out]	Source	The structure to disassemble.
	 */
	SCRIPTDISASSEMBLER_API void DisassembleStructure(UFunction* Source);

	/**
	 * Disassemble all functions in any classes that have matching names.
	 *
	 * @param	InAr	The archive to emit disassembled bytecode to.
	 * @param	ClassnameSubstring	A class must contain this substring to be disassembled.
	 */
	SCRIPTDISASSEMBLER_API static void DisassembleAllFunctionsInClasses(FOutputDevice& Ar, const FString& ClassnameSubstring);
private:

	// Reading functions
	int32 ReadINT(int32& ScriptIndex);
	uint64 ReadQWORD(int32& ScriptIndex);
	uint8 ReadBYTE(int32& ScriptIndex);
	FString ReadName(int32& ScriptIndex);
	uint16 ReadWORD(int32& ScriptIndex);
	float ReadFLOAT(int32& ScriptIndex);
	double ReadDOUBLE(int32& ScriptIndex);
	FVector ReadFVECTOR(int32& ScriptIndex);
	FRotator ReadFROTATOR(int32& ScriptIndex);
	FQuat ReadFQUAT(int32& ScriptIndex);
	FTransform ReadFTRANSFORM(int32& ScriptIndex);
	CodeSkipSizeType ReadSkipCount(int32& ScriptIndex);
	FString ReadString(int32& ScriptIndex);
	FString ReadString8(int32& ScriptIndex);
	FString ReadString16(int32& ScriptIndex);

	EExprToken SerializeExpr(int32& ScriptIndex);
	void ProcessCommon(int32& ScriptIndex, EExprToken Opcode);

	void InitTables();

	template<typename T>
	void Skip(int32& ScriptIndex)
	{
		ScriptIndex += sizeof(T);
	}

	void AddIndent()
	{
		Indents += TEXT("  ");
	}

	void DropIndent()
	{
		// Blah, this is awful
		Indents.LeftInline(Indents.Len() - 2);
	}

	template <typename T>
	T* ReadPointer(int32& ScriptIndex)
	{
		return (T*)ReadQWORD(ScriptIndex);
	}
};
