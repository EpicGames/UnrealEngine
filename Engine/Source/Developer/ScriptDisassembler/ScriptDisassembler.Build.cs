// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ScriptDisassembler : ModuleRules
	{
		public ScriptDisassembler(ReadOnlyTargetRules Target) : base(Target)
        {
			bRequiresImplementModule = false;

			PublicDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject"
			});
        }
	}
}
